// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothGenerator.h"

#include "AnimationRuntime.h"
#include "Animation/AnimSequence.h"
#include "Animation/AttributesRuntime.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "BonePose.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothSimulationProxy.h"
#include "ClothGeneratorComponent.h"
#include "ClothGeneratorProperties.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "FileHelpers.h"
#include "GeometryCache.h"
#include "GeometryCacheCodecV1.h"
#include "GeometryCacheMeshData.h"
#include "GeometryCacheTrackStreamable.h"
#include "GeometryCacheConstantTopologyWriter.h"
#include "IDocumentation.h"
#include "Internationalization/Regex.h"
#include "Misc/AsyncTaskNotification.h"
#include "Misc/Optional.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkeletalMeshModel.h"
#include "SkeletalMeshAttributes.h"
#include "SkeletalRenderPublic.h"
#include "Tasks/Pipe.h"
#include "UObject/SavePackage.h"

DEFINE_LOG_CATEGORY(LogChaosClothGenerator);

#define LOCTEXT_NAMESPACE "ChaosClothGenerator"

namespace UE::Chaos::ClothGenerator
{
	namespace Private
	{
		TArray<int32> ParseFrames(const FString& FramesString)
		{
			TArray<int32> Result;
			static const FRegexPattern AllowedCharsPattern(TEXT("^[-,0-9\\s]+$"));
		
			if (!FRegexMatcher(AllowedCharsPattern, FramesString).FindNext())
			{
			    UE_LOGF(LogChaosClothGenerator, Error, "Input contains invalid characters.");
			    return Result;
			}
		
			static const FRegexPattern SingleNumberPattern(TEXT("^\\s*(\\d+)\\s*$"));
			static const FRegexPattern RangePattern(TEXT("^\\s*(\\d+)\\s*-\\s*(\\d+)\\s*$"));
		
			TArray<FString> Segments;
		    FramesString.ParseIntoArray(Segments, TEXT(","), true);
		    for (const FString& Segment : Segments)
		    {
		    	bool bSegmentValid = false;
		
		    	FRegexMatcher SingleNumberMatcher(SingleNumberPattern, Segment);
		    	if (SingleNumberMatcher.FindNext())
		    	{
		    	    const int32 SingleNumber = FCString::Atoi(*SingleNumberMatcher.GetCaptureGroup(1));
		    	    Result.Add(SingleNumber);
		    	    bSegmentValid = true;
		    	}
		    	else
		    	{
		    		FRegexMatcher RangeMatcher(RangePattern, Segment);
		    		if (RangeMatcher.FindNext())
		    		{
		    		    const int32 RangeStart = FCString::Atoi(*RangeMatcher.GetCaptureGroup(1));
		    		    const int32 RangeEnd = FCString::Atoi(*RangeMatcher.GetCaptureGroup(2));
		
		    		    for (int32 i = RangeStart; i <= RangeEnd; ++i)
		    		    {
		    		        Result.Add(i);
		    		    }
		    		    bSegmentValid = true;
		    		}
		    	}
		    	
		    	if (!bSegmentValid)
		    	{
		    	    UE_LOGF(LogChaosClothGenerator, Error, "Invalid format in segment: %ls", *Segment);
		    	}
		    }
		
			return Result;
		}
		
		TArray<int32> Range(int32 End)
		{
			TArray<int32> Result;
			Result.Reserve(End);
			for (int32 Index = 0; Index < End; ++Index)
			{
				Result.Add(Index);
			}
			return Result;
		}
		
		TArray<uint32> Range(uint32 Start, uint32 End)
		{
			TArray<uint32> Result;
			const uint32 Num = End - Start;
			Result.SetNumUninitialized(Num);
			for (uint32 Index = 0; Index < Num; ++Index)
			{
				Result.Add(Index + Start);
			}
			return Result;
		}
		
		int32 GetNumVertices(const FSkeletalMeshLODRenderData& LODData)
		{
			int32 NumVertices = 0;
			for(const FSkelMeshRenderSection& Section : LODData.RenderSections)
			{
				NumVertices += Section.NumVertices;
			}
			return NumVertices;
		}

		int32 GetNumVertices(const USkeletalMesh& SkeletalMesh)
		{
			const FSkeletalMeshRenderData* RenderData = SkeletalMesh.GetResourceForRendering();
			constexpr int32 LODIndex = 0;
			if (!RenderData || !RenderData->LODRenderData.IsValidIndex(LODIndex))
			{
				return INDEX_NONE;
			}
			const FSkeletalMeshLODRenderData& LODData = RenderData->LODRenderData[LODIndex];
			return GetNumVertices(LODData);
		}
		
		TArrayView<TArray<FVector3f>> ShrinkToValidFrames(const TArrayView<TArray<FVector3f>>& Positions, int32 NumVertices)
		{
			int32 NumValidFrames = 0;
			for (const TArray<FVector3f>& Frame : Positions)
			{
				if (Frame.Num() != NumVertices)
				{
					break;
				}
				++NumValidFrames;
			}
			return TArrayView<TArray<FVector3f>>(Positions.GetData(), NumValidFrames);
		}

		TArray<FVector2f> GetUV0s(const FSkeletalMeshLODRenderData& LODData)
		{
			TArray<FVector2f> UV0s;
			const FStaticMeshVertexBuffer& StaticMeshVertexBuffer = LODData.StaticVertexBuffers.StaticMeshVertexBuffer;
			const int32 NumVertices = StaticMeshVertexBuffer.GetNumVertices();
			UV0s.SetNumZeroed(NumVertices);
			for (int32 Index = 0; Index < NumVertices; ++Index)
			{
				UV0s[Index] = StaticMeshVertexBuffer.GetVertexUV(Index, 0);
			}
			return UV0s;
		}

		TArray<FColor> GetColors(const FSkeletalMeshLODRenderData& LODData, int32 NumVertices)
		{
			TArray<FColor> Colors;
			Colors.SetNum(NumVertices);
			const FColorVertexBuffer& ColorVertexBuffer = LODData.StaticVertexBuffers.ColorVertexBuffer;
			if (ColorVertexBuffer.GetNumVertices() == NumVertices)
			{
				for (int32 Index = 0; Index < NumVertices; ++Index)
				{
					Colors[Index] = ColorVertexBuffer.VertexColor(Index);
				}
			}
			else
			{
				for (int32 Index = 0; Index < NumVertices; ++Index)
				{
					Colors[Index] = FColor::White;
				}
			}
			return Colors;
		}

		TArray<TObjectPtr<UMaterialInterface>> GetMaterialInterfaces(const USkinnedAsset& Asset)
		{
			const TArray<FSkeletalMaterial>& Materials = Asset.GetMaterials();
			TArray<TObjectPtr<UMaterialInterface>> Interfaces;
			Interfaces.SetNum(Materials.Num());
			for (int32 Index = 0; Index < Materials.Num(); ++Index)
			{
				Interfaces[Index] = Materials[Index].MaterialInterface;
			}
			return Interfaces;
		}

		TArray<TInterval<int32>> GetImportPartIntervals(FSkeletalMeshConstAttributes MeshAttributes)
		{
			const int32 NumParts = MeshAttributes.GetNumSourceGeometryParts();
			const FSkeletalMeshAttributes::FSourceGeometryPartVertexOffsetAndCountConstRef OffsetAndCounts = MeshAttributes.GetSourceGeometryPartVertexOffsetAndCounts();
			TArray<TInterval<int32>> Result;
			Result.SetNum(NumParts);
			for (int32 PartIndex = 0; PartIndex < NumParts; ++PartIndex)
			{
				TConstArrayView<int32> OffsetAndCount = OffsetAndCounts.Get(PartIndex);
				const int32 Offset = OffsetAndCount[0];
				const int32 Count = OffsetAndCount[1];
				Result[PartIndex] = TInterval<int32>(Offset, Offset + Count - 1);
			}
			return Result;
		}

		int32 GetPartIndexFromVertexIndex(int32 VertexIndex, const TArray<int32>& MeshToImportVertexMap, const TArray<TInterval<int32>>& ImportPartIntervals)
		{
			if (!MeshToImportVertexMap.IsValidIndex(VertexIndex))
			{
				return INDEX_NONE;
			}
			const int32 ImportIndex = MeshToImportVertexMap[VertexIndex];
			const int32 NumParts = ImportPartIntervals.Num();
			for (int32 PartIndex = 0; PartIndex < NumParts; ++PartIndex)
			{
				const TInterval<int32>& Interval = ImportPartIntervals[PartIndex];
				if (Interval.Contains(ImportIndex))
				{
					return PartIndex;
				}
			}
			return INDEX_NONE;
		}

		TArray<int32> GetVertexToPart(const TArray<int32>& MeshToImportVertexMap, const TArray<TInterval<int32>>& ImportPartIntervals)
		{
			const int32 NumVertices = MeshToImportVertexMap.Num();
			TArray<int32> VertexToPart;
			VertexToPart.SetNum(NumVertices);
			for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
			{
				VertexToPart[VertexIndex] = GetPartIndexFromVertexIndex(VertexIndex, MeshToImportVertexMap, ImportPartIntervals);
			}
			return VertexToPart;
		}

		TArray<TArray<int32>> GetPartToIndices(const TArray<int32>& PartIndices, const int32 NumParts)
		{
			TArray<TArray<int32>> PartToIndices;
			PartToIndices.SetNum(NumParts);
			for (int32 Index = 0; Index < PartIndices.Num(); ++Index)
			{
				const int32 PartIndex = PartIndices[Index];
				PartToIndices[PartIndex].Add(Index);
			}
			return PartToIndices;
		}

		TOptional<TArray<int32>> GetTriangleToPart(const TArray<uint32>& Indices, const TArray<int32>& VertexToPart)
		{
			check(Indices.Num() % 3 == 0);
			const int32 NumTriangles = Indices.Num() / 3;
			TOptional<TArray<int32>> None;
			TArray<int32> TriangleToPart;
			TriangleToPart.SetNum(NumTriangles);
			for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
			{
				int32 PartIndex = INDEX_NONE;
				for (int32 Corner = 0; Corner < 3; ++Corner)
				{
					const int32 VertexIndex = Indices[TriangleIndex * 3 + Corner];
					const int32 CandidatePartIndex = VertexToPart[VertexIndex];
					if (PartIndex == INDEX_NONE)
					{
						PartIndex = CandidatePartIndex;
					}
					else if (PartIndex != CandidatePartIndex)
					{
						return None;
					}
				}
				TriangleToPart[TriangleIndex] = PartIndex;
			}
			return TriangleToPart;
		}

		TArray<int32> GetIndicesFromTriangles(const TArray<int32>& Triangles)
		{
			TArray<int32> Indices;
			Indices.SetNum(Triangles.Num() * 3);
			for (int32 TriangleIndex = 0; TriangleIndex < Triangles.Num(); ++TriangleIndex)
			{
				const int32 Triangle = Triangles[TriangleIndex];
				Indices[TriangleIndex * 3] = Triangle * 3;
				Indices[TriangleIndex * 3 + 1] = Triangle * 3 + 1;
				Indices[TriangleIndex * 3 + 2] = Triangle * 3 + 2;
			}
			return Indices;
		}

		template<typename T>
		TArray<T> Gather(const TArray<T>& Array, const TArray<int32>& Indices)
		{
			TArray<T> Result;
			Result.SetNum(Indices.Num());
			for (int32 Index = 0; Index < Indices.Num(); ++Index)
			{
				Result[Index] = Array[Indices[Index]];
			}
			return Result;
		}

		template<typename T1, typename T2>
		TArray<T2> Map(const TArray<T1>& Array, const TArray<T2>& Mapper)
		{
			TArray<int32> Result;
			Result.SetNum(Array.Num());
			for (int32 Index = 0; Index < Array.Num(); ++Index)
			{
				Result[Index] = Mapper[Array[Index]];
			}
			return Result;
		}

		TArray<int32> InverseMap(const TArray<int32>& Map, const int32 Num)
		{
			TArray<int32> InversedMap;
			InversedMap.SetNum(Num);
			for (int32 Index = 0; Index < Map.Num(); ++Index)
			{
				InversedMap[Map[Index]] = Index;
			}
			return InversedMap;
		}

		TArray<TArray<int32>> GetSectionToTriangleIndices(const FSkeletalMeshLODRenderData& LODData, const TArray<int32>& TriangleIndices)
		{
			const TArray<FSkelMeshRenderSection>& Sections = LODData.RenderSections;
			const int32 NumSections = Sections.Num();
			TArray<TArray<int32>> SectionToTriangleIndices;
			SectionToTriangleIndices.SetNumZeroed(NumSections);
			for (int32 TriangleIndex : TriangleIndices)
			{
				for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
				{
					const FSkelMeshRenderSection& Section = Sections[SectionIndex];
					const int32 Start = Section.BaseIndex;
					const int32 End = Start + Section.NumTriangles - 1;
					if (Start <= TriangleIndex && TriangleIndex <= End)
					{
						SectionToTriangleIndices[SectionIndex].Add(TriangleIndex);
						break;
					}
				}
			}

			return SectionToTriangleIndices;
		}

		template<typename SrcT, typename DstT>
		TArray<DstT> ConvertArray(const TArray<SrcT>& SrcArray)
		{
			TArray<DstT> DstArray;
			DstArray.SetNumUninitialized(SrcArray.Num());
			for (int32 Index = 0; Index < SrcArray.Num(); ++Index)
			{
				DstArray[Index] = SrcArray[Index];
			}
			return DstArray;
		}

		TArray<uint32> OffsetElements(const TArray<uint32>& Array, int32 Offset)
		{
			TArray<uint32> OffsetArray;
			OffsetArray.SetNumUninitialized(Array.Num());
			for (int32 Index = 0; Index < Array.Num(); ++Index)
			{
				check((int32)Array[Index] + Offset >= 0);
				OffsetArray[Index] = Array[Index] + Offset;
			}
			return OffsetArray;
		}

		int32 AddTrackWritersFromSkeletalMesh(UE::GeometryCacheHelpers::FGeometryCacheConstantTopologyWriter& Writer, const USkeletalMesh& SkeletalMesh)
		{
			const FSkeletalMeshRenderData* RenderData = SkeletalMesh.GetResourceForRendering();
			constexpr int32 LODIndex = 0;
			if (!RenderData || !RenderData->LODRenderData.IsValidIndex(LODIndex))
			{
				UE_LOGF(LogChaosClothGenerator, Error, "SkeletalMesh has no render data. Failed to create track writers.");
				return 0;
			}
			const FSkeletalMeshLODRenderData& LODData = RenderData->LODRenderData[LODIndex];

			const FMeshDescription* MeshDescription = SkeletalMesh.GetMeshDescription(LODIndex);
			if (!MeshDescription || MeshDescription->IsEmpty())
			{
				UE_LOGF(LogChaosClothGenerator, Error, "SkeletalMesh has no mesh description. Failed to create track writers.");
				return 0;
			}
			FSkeletalMeshConstAttributes MeshAttributes(*MeshDescription);
			if (!MeshAttributes.HasSourceGeometryParts())
			{
				UE_LOGF(LogChaosClothGenerator, Error, "SkeletalMesh has no source geometry parts. Failed to create track writers.");
				return 0;
			}
			const int32 NumParts = MeshAttributes.GetNumSourceGeometryParts();
			if (NumParts <= 0)
			{
				UE_LOGF(LogChaosClothGenerator, Error, "SkeletalMesh has no source geometry parts. Failed to create track writers.");
				return 0;
			}

			const FSkeletalMeshModel* const ImportModel = SkeletalMesh.GetImportedModel();
			if (!ImportModel || !ImportModel->LODModels.IsValidIndex(LODIndex))
			{
				UE_LOGF(LogChaosClothGenerator, Error, "SkeletalMesh has no import model. Failed to create track writers.");
				return 0;
			}
			const TArray<int32>& MeshToImportVertexMap = ImportModel->LODModels[LODIndex].MeshToImportVertexMap;
			const int32 NumVertices = GetNumVertices(LODData);
			TArray<uint32> Indices;
			LODData.MultiSizeIndexContainer.GetIndexBuffer(Indices);
			check(Indices.Num() % 3 == 0);

			const TArray<FVector2f> UVs = GetUV0s(LODData);
			check(UVs.Num() == NumVertices);
			const TArray<FColor> Colors = GetColors(LODData, NumVertices);
			check(Colors.Num() == NumVertices);

			const FSkeletalMeshAttributes::FSourceGeometryPartNameConstRef PartNames = MeshAttributes.GetSourceGeometryPartNames();
			const TArray<TInterval<int32>> ImportPartIntervals = GetImportPartIntervals(MeshAttributes);

			const TArray<int32> VertexToPart = GetVertexToPart(MeshToImportVertexMap, ImportPartIntervals);
			const TArray<TArray<int32>> PartToVertices = GetPartToIndices(VertexToPart, NumParts);
			const TOptional<TArray<int32>> OptionalTriangleToPart = GetTriangleToPart(Indices, VertexToPart);
			if (!OptionalTriangleToPart.IsSet())
			{
				UE_LOGF(LogChaosClothGenerator, Error, "Assuming all vertices in the same triangle are in the same part but some triangles are not. Failed to create track writers.");
				return 0;
			}
			const TArray<int32>& TriangleToPart = OptionalTriangleToPart.GetValue();
			const TArray<TArray<int32>> PartToTriangles = GetPartToIndices(TriangleToPart, NumParts);
			for (int32 PartIndex = 0; PartIndex < NumParts; ++PartIndex)
			{
				const FName TrackName = PartNames.Get(PartIndex);
				using FTrackWriter = UE::GeometryCacheHelpers::FGeometryCacheConstantTopologyWriter::FTrackWriter;
				FTrackWriter& TrackWriter = Writer.AddTrackWriter(TrackName);
				const TArray<int32>& VertexIndices = PartToVertices[PartIndex];
				const TArray<int32> InverseVertexMap = InverseMap(VertexIndices, NumVertices);
				const TArray<int32>& TriangleIndices = PartToTriangles[PartIndex];
				const int32 StartImportedVertex = ImportPartIntervals[PartIndex].Min;
				TrackWriter.UVs = Gather(UVs, VertexIndices);
				TrackWriter.Colors = Gather(Colors, VertexIndices);
				TrackWriter.ImportedVertexNumbers = OffsetElements(ConvertArray<int32, uint32>(Gather(MeshToImportVertexMap, VertexIndices)), -StartImportedVertex);
				TrackWriter.SourceVertexIndices = VertexIndices;

				const TArray<TArray<int32>> SectionToTriangleIndices = GetSectionToTriangleIndices(LODData, TriangleIndices);
				TArray<uint32> PartIndices;
				PartIndices.Reserve(TriangleIndices.Num() * 3);
				for (int32 SectionIndex = 0; SectionIndex < SectionToTriangleIndices.Num(); ++SectionIndex)
				{
					const TArray<int32>& SectionTriangleIndices = SectionToTriangleIndices[SectionIndex];
					if (SectionTriangleIndices.IsEmpty())
					{
						continue;
					}
					TArray<uint32> SectionIndices = ConvertArray<int32, uint32>(Map(Gather(Indices, GetIndicesFromTriangles(SectionTriangleIndices)), InverseVertexMap));
					check(SectionIndices.Num() % 3 == 0);
					FGeometryCacheMeshBatchInfo BatchInfo;
					BatchInfo.StartIndex = PartIndices.Num();
					BatchInfo.NumTriangles = SectionIndices.Num() / 3;
					BatchInfo.MaterialIndex = LODData.RenderSections[SectionIndex].MaterialIndex;
					TrackWriter.BatchesInfo.Add(BatchInfo);
					PartIndices.Append(MoveTemp(SectionIndices));
				}
				TrackWriter.Indices = PartIndices;
			}
			Writer.AddMaterials(GetMaterialInterfaces(SkeletalMesh));
			return NumParts;
		}

		TArray<TArray<FVector3f>> Gather(const TArrayView<TArray<FVector3f>>& Positions, const TArray<int32>& Indices)
		{
			if (Indices.IsEmpty())
			{
				return TArray<TArray<FVector3f>>();
			}
			const int32 NumFrames = Positions.Num();
			TArray<TArray<FVector3f>> Result;
			Result.SetNum(NumFrames);
			for (int32 Frame = 0; Frame < NumFrames; ++Frame)
			{
				const TArray<FVector3f>& FramePositions = Positions[Frame];
				Result[Frame] = Gather(FramePositions, Indices);
			}
			return Result;
		}
		
		void SaveGeometryCache(UGeometryCache& GeometryCache, const USkeletalMesh& SkeletalMesh, TArrayView<TArray<FVector3f>> PositionsToMoveFrom)
		{
			constexpr int32 LODIndex = 0;
			const int32 NumVertices = GetNumVertices(SkeletalMesh);
			PositionsToMoveFrom = ShrinkToValidFrames(PositionsToMoveFrom, NumVertices);
			if (PositionsToMoveFrom.IsEmpty())
			{
				UE_LOGF(LogChaosClothGenerator, Error, "There is no valid data in PositionsToMoveFrom. No geometry cache is saved.");
				return;
			}
		
			using UE::GeometryCacheHelpers::FGeometryCacheConstantTopologyWriter;
			using UE::GeometryCacheHelpers::AddTrackWriterFromSkinnedAsset;
			using FTrackWriter = FGeometryCacheConstantTopologyWriter::FTrackWriter;
			FGeometryCacheConstantTopologyWriter Writer(GeometryCache);

			const int32 NumTracks = AddTrackWritersFromSkeletalMesh(Writer, SkeletalMesh);
			if (NumTracks == 0)
			{
				UE_LOGF(LogChaosClothGenerator, Error, "Failed to add track writers. No geometry cache is saved.");
				return;
			}
			for (int32 Index = 0; Index < NumTracks; ++Index)
			{
				FTrackWriter& TrackWriter = Writer.GetTrackWriter(Index);
				check(TrackWriter.SourceVertexIndices.IsSet());
				TrackWriter.WriteAndClose(Gather(PositionsToMoveFrom, TrackWriter.SourceVertexIndices.GetValue()));
			}
		}
		
		class FTimeScope
		{
		public:
			explicit FTimeScope(FString InName)
				: Name(MoveTemp(InName))
				, StartTime(FDateTime::UtcNow())
			{
			}
			~FTimeScope()
			{
				const FTimespan Duration = FDateTime::UtcNow() - StartTime;
				UE_LOGF(LogChaosClothGenerator, Log, "%ls took %f secs", *Name, Duration.GetTotalSeconds());
			}
		private:
			FString Name;
			FDateTime StartTime;
		};
		
		void SavePackage(UObject& Object)
		{
			TArray<UPackage*> PackagesToSave = { Object.GetOutermost() };
			constexpr bool bCheckDirty = false;
			constexpr bool bPromptToSave = false;
			FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirty, bPromptToSave);
		}

		bool AreAssetsConsistent(const USkinnedAsset& SkeletalMeshAsset, const UChaosClothAsset& ClothAsset)
		{
			if (!SkeletalMeshAsset.IsA<USkeletalMesh>())
			{
				UE_LOGF(LogChaosClothGenerator, Warning, "SkeletalMeshAsset is not a SkeletalMesh.");
				return false;
			}
			constexpr int32 LODIndex = 0;
			const TOptional<TArray<int32>> None;
			const FSkeletalMeshModel* const MLDModel = SkeletalMeshAsset.GetImportedModel();
			if (!MLDModel || !MLDModel->LODModels.IsValidIndex(LODIndex))
			{
				return false;
			}
			const FSkeletalMeshLODModel& MLDLOD = MLDModel->LODModels[LODIndex];
			const TArray<int32>& Map = MLDLOD.MeshToImportVertexMap;
			if (Map.IsEmpty())
			{
				UE_LOGF(LogChaosClothGenerator, Warning, "MeshToImportVertexMap is empty. MLDeformer Asset should be an imported SkeletalMesh (e.g. from fbx).");
				return false;
			}
			const FSkeletalMeshModel* const ClothModel = ClothAsset.GetImportedModel();
			if (!ClothModel || !ClothModel->LODModels.IsValidIndex(LODIndex))
			{
				UE_LOGF(LogChaosClothGenerator, Warning, "ClothAsset has no imported model.");
				return false;
			}
			const FSkeletalMeshLODModel& ClothLOD = ClothModel->LODModels[LODIndex];

			if (MLDLOD.NumVertices != ClothLOD.NumVertices || MLDLOD.Sections.Num() != ClothLOD.Sections.Num())
			{
				UE_LOGF(LogChaosClothGenerator, Warning, "SkeletalMeshAsset and ClothAsset have different number of vertices or sections. Check if the assets have the same mesh.");
				return false;
			}

			for (int32 SectionIndex = 0; SectionIndex < MLDLOD.Sections.Num(); ++SectionIndex)
			{
				const FSkelMeshSection& MLDSection = MLDLOD.Sections[SectionIndex];
				const FSkelMeshSection& ClothSection = ClothLOD.Sections[SectionIndex];
				if (MLDSection.NumVertices != ClothSection.NumVertices)
				{
					UE_LOGF(LogChaosClothGenerator, Warning, "SkeletalMeshAsset and ClothAsset have different number of vertices in section %d. Check if the assets have the same mesh.", SectionIndex);
					return false;
				}
				for (int32 VertexIndex = 0; VertexIndex < MLDSection.NumVertices; ++VertexIndex)
				{
					const FVector3f& MLDPosition = MLDSection.SoftVertices[VertexIndex].Position;
					const FVector3f& ClothPosition = ClothSection.SoftVertices[VertexIndex].Position;
					if (!MLDPosition.Equals(ClothPosition, UE_KINDA_SMALL_NUMBER))
					{
						UE_LOGF(LogChaosClothGenerator, Warning, "SkeletalMeshAsset and ClothAsset have different vertex positions. Check if the assets have the same vertex order.");
						return false;
					}
				}
			}

			return true;
		}
	};

	void FChaosClothGenerator::Tick(float DeltaTime)
	{
		if (PendingAction == EClothGeneratorActions::StartGenerate)
		{
			StartGenerate();
		}
		else if (PendingAction == EClothGeneratorActions::TickGenerate)
		{
			TickGenerate();
		}
	}

	TStatId FChaosClothGenerator::GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FChaosClothGenerator, STATGROUP_Tickables);
	}

	template<typename TaskType>
	class FChaosClothGenerator::TTaskRunner : public FNonAbandonableTask
	{
	public:
		TTaskRunner(TUniquePtr<TaskType> InTask)
			: Task(MoveTemp(InTask))
		{
		}
	
		void DoWork()
		{
			if (Task)
			{
				Task->DoWork();
			}
		}
	
		FORCEINLINE TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(TTaskRunner, STATGROUP_ThreadPoolAsyncTasks);
		}
	
	private:
		TUniquePtr<TaskType> Task;
	};
	
	struct FChaosClothGenerator::FSimResource
	{
		TStrongObjectPtr<UClothGeneratorComponent> ClothComponent = nullptr;
		TSharedPtr<FProxy> Proxy;
		TUniquePtr<UE::Tasks::FPipe> Pipe;
		FEvent* SkinEvent = nullptr;
		std::atomic<bool> bNeedsSkin = false;
		TArrayView<TArray<FVector3f>> SimulatedPositions;
		std::atomic<int32>* NumSimulatedFrames = nullptr;
		std::atomic<bool>* bCancelled = nullptr;

		bool IsCancelled() const { return !bCancelled || bCancelled->load(); }
		void FinishFrame()
		{
			if (NumSimulatedFrames)
			{
				++(*NumSimulatedFrames); 
			}
		}
	};
	
	struct FChaosClothGenerator::FTaskResource
	{
		TArray<FSimResource> SimResources;

		TUniquePtr<FExecuterType> Executer;
		TUniquePtr<FAsyncTaskNotification> Notification;
		FDateTime StartTime;
		FDateTime LastUpdateTime;

		TArray<int32> FramesToSimulate;
		TArray<TArray<FVector3f>> SimulatedPositions;
		UGeometryCache* Cache = nullptr;

		std::atomic<int32> NumSimulatedFrames = 0;
		std::atomic<bool> bCancelled = false;

		UWorld* World = nullptr;
	
		bool AllocateSimResources_GameThread(UChaosClothAsset& Asset, int32 Num);
		void FreeSimResources_GameThread();
		void FlushRendering();
		void Cancel();
	};

	class FChaosClothGenerator::FLaunchSimsTask
	{
	public: 
		FLaunchSimsTask(FTaskResource& InTaskResource, TStrongObjectPtr<UClothGeneratorProperties> InProperties)
			: TaskResource(InTaskResource)
			, SimResources(InTaskResource.SimResources)
			, Properties(InProperties)
		{}
	
		void DoWork();
	
	private:
		using FPipe = UE::Tasks::FPipe;
	
		enum class ESaveType
		{
			LastStep,
			EveryStep,
		};
	
		void Simulate(FSimResource &SimResource, int32 AnimFrame, int32 CacheFrame) const;
		void PrepareAnimationSequence();
		void RestoreAnimationSequence();
		TArray<FTransform> GetBoneTransforms(UChaosClothComponent& InClothComponent, int32 Frame) const;
		TArray<FVector3f> GetRenderPositions(FSimResource& SimResource) const;
	
		FTaskResource& TaskResource;
		TArray<FSimResource>& SimResources;
		TStrongObjectPtr<UClothGeneratorProperties> Properties;
		EAnimInterpolationType InterpolationTypeBackup = EAnimInterpolationType::Linear;
	};

	bool FChaosClothGenerator::FTaskResource::AllocateSimResources_GameThread(UChaosClothAsset& Asset, int32 Num)
	{
		World = UWorld::CreateWorld(EWorldType::None, false);
		SimResources.SetNum(Num);
		for(int32 Index = 0; Index < Num; ++Index)
		{
			UClothGeneratorComponent* const CopyComponent = NewObject<UClothGeneratorComponent>();
			CopyComponent->SetAsset(&Asset);
			CopyComponent->RegisterComponentWithWorld(World);
	
			USkinnedMeshComponent* const PoseComponent = CopyComponent->LeaderPoseComponent.Get() ? CopyComponent->LeaderPoseComponent.Get() : CopyComponent;
			constexpr int32 LODIndex = 0;
			PoseComponent->SetForcedLOD(LODIndex + 1);
			PoseComponent->UpdateLODStatus();
			PoseComponent->RefreshBoneTransforms(nullptr);
			CopyComponent->bRenderStatic = false;
			constexpr bool bRecreateRenderStateImmediately = true;
			CopyComponent->SetCPUSkinningEnabled(true, bRecreateRenderStateImmediately);
			CopyComponent->ResumeSimulation();
	
			FSimResource& SimResource = SimResources[Index];
			SimResource.ClothComponent = TStrongObjectPtr<UClothGeneratorComponent>(CopyComponent);
			SimResource.Proxy = CopyComponent->GetProxy().Pin();
			check(SimResource.Proxy != nullptr);
			SimResource.Pipe = MakeUnique<UE::Tasks::FPipe>(*FString::Printf(TEXT("SimPipe:%d"), Index));
			SimResource.SkinEvent = FPlatformProcess::GetSynchEventFromPool();
			SimResource.bNeedsSkin.store(false);
	
			SimResource.SimulatedPositions = TArrayView<TArray<FVector3f>>(SimulatedPositions);
			SimResource.NumSimulatedFrames = &NumSimulatedFrames;
			SimResource.bCancelled = &bCancelled;
	
			if (SimResource.Proxy == nullptr || SimResource.Pipe == nullptr)
			{
				UE_LOGF(LogChaosClothGenerator, Error, "Failed to allocate simulation resources");
				return false;
			}
		}
		return true;
	}
	
	void FChaosClothGenerator::FTaskResource::FreeSimResources_GameThread()
	{
		if (Executer.IsValid())
		{
			Executer->EnsureCompletion();
		}
		for (FSimResource& SimResource : SimResources)
		{
			FPlatformProcess::ReturnSynchEventToPool(SimResource.SkinEvent);
			SimResource.Pipe.Reset();
			SimResource.ClothComponent->UnregisterComponent();
			SimResource.ClothComponent->DestroyComponent();
		}
		SimResources.Empty();
		World->DestroyWorld(false);
	}
	
	void FChaosClothGenerator::FTaskResource::FlushRendering()
	{
		// Copy bNeedsSkin
		TArray<bool> NeedsSkin;
		NeedsSkin.SetNum(SimResources.Num());
		bool bAnyNeedsSkin = false;
		for (int32 Index = 0; Index < SimResources.Num(); ++Index)
		{
			const bool bNeedsSkin = SimResources[Index].bNeedsSkin.load();
			bAnyNeedsSkin |= bNeedsSkin;
			NeedsSkin[Index] = bNeedsSkin;
		}
	
		if (bAnyNeedsSkin)
		{
			FlushRenderingCommands();
			for (int32 Index = 0; Index < SimResources.Num(); ++Index)
			{
				if (NeedsSkin[Index])
				{
					SimResources[Index].bNeedsSkin.store(false);
					SimResources[Index].SkinEvent->Trigger();
				}
			}
		}
	}

	void FChaosClothGenerator::FTaskResource::Cancel()
	{
		bCancelled.store(true);
	}

	void FChaosClothGenerator::FLaunchSimsTask::DoWork()
	{	
		const int32 NumFrames = TaskResource.FramesToSimulate.Num();
		PrepareAnimationSequence();
	
		const int32 NumThreads = Properties->bDebug ? 1 : Properties->NumThreads;
	
		for (int32 Frame = 0; Frame < NumFrames; Frame++)
		{
			if (!TaskResource.bCancelled.load())
			{
				const int32 ThreadIdx = Frame % NumThreads;
				const int32 AnimFrame = TaskResource.FramesToSimulate[Frame];
	
				FSimResource& SimResource = SimResources[ThreadIdx];
				SimResource.Pipe->Launch(*FString::Printf(TEXT("SimFrame:%d"), AnimFrame), [this, &SimResource, AnimFrame, Frame]()
				{ 
					FMemMark Mark(FMemStack::Get());
					Simulate(SimResource, AnimFrame, Frame);
				});
			}
			else
			{
				break;
			}
		}
	
		for (FSimResource& SimResource : SimResources)
		{
			SimResource.Pipe->WaitUntilEmpty();
		}
	
		RestoreAnimationSequence();
	}

	void FChaosClothGenerator::FLaunchSimsTask::Simulate(FSimResource &SimResource, int32 AnimFrame, int32 CacheFrame) const
	{
		UClothGeneratorComponent& TaskComponent = *SimResource.ClothComponent;
		FProxy& DataGenerationProxy = *SimResource.Proxy;
	
		const float TimeStep = Properties->TimeStep;
		const int32 NumSteps = Properties->NumSteps;
		const ESaveType SaveType = Properties->bDebug ? ESaveType::EveryStep : ESaveType::LastStep;
	
		const TArray<FTransform> Transforms = GetBoneTransforms(TaskComponent, AnimFrame);
		TaskComponent.Pose(Transforms);
		TaskComponent.ForceNextUpdateTeleportAndReset();
		DataGenerationProxy.PreProcess_Internal(TimeStep);
		DataGenerationProxy.InitializeConfigs();
		bool bCancelled = false;
		for (int32 Step = 0; Step < NumSteps; ++Step)
		{
			if (SimResource.IsCancelled())
			{
				bCancelled = true;
				break;
			}
			else 
			{
				DataGenerationProxy.Tick();
	
				// Clear any reset flags at the end of the first step
				if (Step == 0 && NumSteps > 1)
				{
					TaskComponent.ResetTeleportMode();
					DataGenerationProxy.PreProcess_Internal(TimeStep);
				}
	
				if (SaveType == ESaveType::EveryStep)
				{
					DataGenerationProxy.WriteSimulationData();
					SimResource.SimulatedPositions[Step] = GetRenderPositions(SimResource);
				}
			}
		}
	
		if (SaveType == ESaveType::LastStep && !bCancelled)
		{
			DataGenerationProxy.WriteSimulationData();
			SimResource.SimulatedPositions[CacheFrame] = GetRenderPositions(SimResource);
		}

		SimResource.FinishFrame();
	}
	
	void FChaosClothGenerator::FLaunchSimsTask::PrepareAnimationSequence()
	{
		TObjectPtr<UAnimSequence> AnimationSequence = Properties->AnimationSequence;
		if (AnimationSequence)
		{
			InterpolationTypeBackup = AnimationSequence->Interpolation;
			AnimationSequence->Interpolation = EAnimInterpolationType::Step;
		}
	}
	
	void FChaosClothGenerator::FLaunchSimsTask::RestoreAnimationSequence()
	{
		TObjectPtr<UAnimSequence> AnimationSequence = Properties->AnimationSequence;
		if (AnimationSequence)
		{
			AnimationSequence->Interpolation = InterpolationTypeBackup;
		}
	}
	
	TArray<FTransform> FChaosClothGenerator::FLaunchSimsTask::GetBoneTransforms(UChaosClothComponent& InClothComponent, int32 Frame) const
	{
		const UAnimSequence* AnimationSequence = Properties->AnimationSequence;
		const double Time = FMath::Clamp(AnimationSequence->GetSamplingFrameRate().AsSeconds(Frame), 0., (double)AnimationSequence->GetPlayLength());
		FAnimExtractContext ExtractionContext(Time);
	
		UChaosClothAsset* const ClothAsset = Cast<UChaosClothAsset>(InClothComponent.GetAsset());
		const FReferenceSkeleton* const ReferenceSkeleton = ClothAsset ? &ClothAsset->GetRefSkeleton() : nullptr;
		USkeleton* const Skeleton = ClothAsset ? ClothAsset->GetSkeleton() : nullptr;
		const int32 NumBones = ReferenceSkeleton ? ReferenceSkeleton->GetNum() : 0;

		if (!ReferenceSkeleton || !Skeleton || NumBones == 0)
		{
			return TArray<FTransform>();
		}
	
		TArray<uint16> BoneIndices;
		BoneIndices.SetNumUninitialized(NumBones);
		for (int32 Index = 0; Index < NumBones; ++Index)
		{
			BoneIndices[Index] = (uint16)Index;
		}
	
		FBoneContainer BoneContainer;
		BoneContainer.SetUseRAWData(true);
		BoneContainer.InitializeTo(BoneIndices, UE::Anim::FCurveFilterSettings(), *Skeleton);
	
		FCompactPose OutPose;
		OutPose.SetBoneContainer(&BoneContainer);
		FBlendedCurve OutCurve;
		OutCurve.InitFrom(BoneContainer);
		UE::Anim::FStackAttributeContainer TempAttributes;
	
		FAnimationPoseData AnimationPoseData(OutPose, OutCurve, TempAttributes);
		AnimationSequence->GetAnimationPose(AnimationPoseData, ExtractionContext);

		const FTransform RootTransform = AnimationSequence->ExtractRootTrackTransform(FAnimExtractContext(Time), nullptr);

		TArray<FTransform> BoneTransforms;
		BoneTransforms.SetNumZeroed(NumBones);
		const FSkeletonToMeshLinkup& LinkupTable = Skeleton->FindOrAddMeshLinkupData(ClothAsset);
		const TArray<int32>& BoneMap = LinkupTable.SkeletonToMeshTable;
		check(BoneMap.Num() == NumBones);
		for (int32 Index = 0; Index < NumBones; ++Index)
		{
			const FCompactPoseBoneIndex CompactIndex = BoneContainer.MakeCompactPoseIndex(FMeshPoseBoneIndex(Index));
			check(BoneTransforms.IsValidIndex(BoneMap[Index]));
			BoneTransforms[BoneMap[Index]] = AnimationPoseData.GetPose()[CompactIndex];
		}
		TArray<FTransform> ComponentSpaceTransforms;
		FAnimationRuntime::FillUpComponentSpaceTransforms(*ReferenceSkeleton, BoneTransforms, ComponentSpaceTransforms);
		return ComponentSpaceTransforms;
	}
	
	TArray<FVector3f> FChaosClothGenerator::FLaunchSimsTask::GetRenderPositions(FSimResource& SimResource) const
	{
		check(SimResource.ClothComponent);
		TArray<FVector3f> Positions;
		TArray<FFinalSkinVertex> OutVertices;
		// This could potentially be slow. 
		SimResource.ClothComponent->RecreateRenderState_Concurrent();
		SimResource.bNeedsSkin.store(true);
		SimResource.SkinEvent->Wait();
		
		SimResource.ClothComponent->GetCPUSkinnedCachedFinalVertices(OutVertices);
		Positions.SetNum(OutVertices.Num());
		for (int32 Index = 0; Index < OutVertices.Num(); ++Index)
		{
			Positions[Index] = OutVertices[Index].Position;
		}
		return Positions;
	}
	
	FChaosClothGenerator::FChaosClothGenerator()
	{
		Properties = TStrongObjectPtr<UClothGeneratorProperties>(NewObject<UClothGeneratorProperties>());
	}

	FChaosClothGenerator::~FChaosClothGenerator()
	{
		if (TaskResource != nullptr)
		{
			TaskResource->FreeSimResources_GameThread();
		}
	}

	void FChaosClothGenerator::StartGenerate()
	{
		check(PendingAction == EClothGeneratorActions::StartGenerate);
		if (Properties->ClothAsset == nullptr)
		{
			UE_LOGF(LogChaosClothGenerator, Error, "ClothAsset is null.");
			PendingAction = EClothGeneratorActions::NoAction;
			return;
		}
		if (Properties->SkeletalMeshAsset == nullptr)
		{
			UE_LOGF(LogChaosClothGenerator, Error, "SkeletalMeshAsset is null.");
			PendingAction = EClothGeneratorActions::NoAction;
			return;
		}
		if (Properties->AnimationSequence == nullptr)
		{
			UE_LOGF(LogChaosClothGenerator, Error, "AnimationSequence is null.");
			PendingAction = EClothGeneratorActions::NoAction;
			return;
		}
		UGeometryCache* const Cache = GetCache();
		if (Cache == nullptr)
		{
			UE_LOGF(LogChaosClothGenerator, Error, "Cannot find or create geometry cache.");
			PendingAction = EClothGeneratorActions::NoAction;
			return;
		}
		if (TaskResource != nullptr)
		{
			UE_LOGF(LogChaosClothGenerator, Error, "Previous generation is still running.");
			PendingAction = EClothGeneratorActions::NoAction;
			return;
		}

		using UE::Chaos::ClothGenerator::Private::AreAssetsConsistent;
		if (!AreAssetsConsistent(*Properties->SkeletalMeshAsset, *Properties->ClothAsset))
		{
			PendingAction = EClothGeneratorActions::NoAction;
			return;
		}
		TaskResource = MakeUnique<FTaskResource>();

		using Private::ParseFrames;
		using Private::Range;
		TaskResource->FramesToSimulate = Properties->bDebug 
			? TArray<int32>{ (int32)Properties->DebugFrame } 
			: (Properties->FramesToSimulate.Len() > 0
				? ParseFrames(Properties->FramesToSimulate) 
				: Range(Properties->AnimationSequence->GetNumberOfSampledKeys()));
		const int32 NumFrames = TaskResource->FramesToSimulate.Num();
		if (NumFrames == 0)
		{
			PendingAction = EClothGeneratorActions::NoAction;
			return;
		}
		TaskResource->SimulatedPositions.SetNum(Properties->bDebug ? Properties->NumSteps : NumFrames);

		if (!TaskResource->AllocateSimResources_GameThread(*Properties->ClothAsset, Properties->NumThreads))
		{
			PendingAction = EClothGeneratorActions::NoAction;
			return;
		}
		TaskResource->Cache = Cache;

		TUniquePtr<FLaunchSimsTask> Task = MakeUnique<FLaunchSimsTask>(*TaskResource, Properties);
		TaskResource->Executer = MakeUnique<FExecuterType>(MoveTemp(Task));
		TaskResource->Executer->StartBackgroundTask();
	
		FAsyncTaskNotificationConfig NotificationConfig;
		NotificationConfig.TitleText = LOCTEXT("SimulateCloth", "Simulating Cloth");
		NotificationConfig.ProgressText = FText::FromString(TEXT("0%"));
		NotificationConfig.bCanCancel = true;
		NotificationConfig.bKeepOpenOnSuccess = true;
		NotificationConfig.bKeepOpenOnFailure = true;
		TaskResource->Notification = MakeUnique<FAsyncTaskNotification>(NotificationConfig);
		TaskResource->StartTime = FDateTime::UtcNow();
		TaskResource->LastUpdateTime = TaskResource->StartTime;
	
		PendingAction = EClothGeneratorActions::TickGenerate;
	}
	
	void FChaosClothGenerator::TickGenerate()
	{
		check(PendingAction == EClothGeneratorActions::TickGenerate && TaskResource != nullptr);
			
		bool bFinished = false;
		const bool bCancelled = TaskResource->Notification->GetPromptAction() == EAsyncTaskNotificationPromptAction::Cancel;
		if (TaskResource->Executer->IsDone())
		{
			bFinished = true;
		}
		else if (bCancelled)
		{
			TaskResource->Cancel();
			bFinished = true;
		}
			
		if (!bFinished)
		{
			TaskResource->FlushRendering();
			const FDateTime CurrentTime = FDateTime::UtcNow();
			const double SinceLastUpdate = (CurrentTime - TaskResource->LastUpdateTime).GetTotalSeconds();
			if (SinceLastUpdate < 0.2)
			{
				return;
			}
			
			const int32 NumSimulatedFrames = TaskResource->NumSimulatedFrames.load();
			const int32 NumTotalFrames = TaskResource->FramesToSimulate.Num();
			const FText ProgressMessage = FText::FromString(FString::Printf(TEXT("Finished %d/%d, %.1f%%"), NumSimulatedFrames, NumTotalFrames, 100.0 * NumSimulatedFrames / NumTotalFrames));
			TaskResource->Notification->SetProgressText(ProgressMessage);
			TaskResource->LastUpdateTime = CurrentTime;
		}
		else
		{
			FreeTaskResource(bCancelled);
			PendingAction = EClothGeneratorActions::NoAction;
		}
	}

	UClothGeneratorProperties& FChaosClothGenerator::GetProperties() const
	{
		return *Properties;
	}
	
	void FChaosClothGenerator::RequestAction(EClothGeneratorActions ActionType)
	{
		if (PendingAction != EClothGeneratorActions::NoAction)
		{
			return;
		}
		PendingAction = ActionType;
	}
	
	
	UGeometryCache* FChaosClothGenerator::GetCache() const
	{
		return Properties->bDebug ? Properties->DebugCache : Properties->SimulatedCache;
	}
	
	void FChaosClothGenerator::FreeTaskResource(bool bCancelled)
	{
		TaskResource->Notification->SetProgressText(LOCTEXT("Finishing", "Finishing, please wait"));
		TaskResource->FreeSimResources_GameThread();
		const FDateTime CurrentTime = FDateTime::UtcNow();
		UE_LOGF(LogChaosClothGenerator, Log, "Training finished in %f seconds", (CurrentTime - TaskResource->StartTime).GetTotalSeconds());
	
		{
			UE::Chaos::ClothGenerator::Private::FTimeScope TimeScope(TEXT("Saving"));
	
			using UE::Chaos::ClothGenerator::Private::SaveGeometryCache;
			using UE::Chaos::ClothGenerator::Private::SavePackage;
			const USkeletalMesh* const SkeletalMesh = Cast<const USkeletalMesh>(Properties->SkeletalMeshAsset);
			check(SkeletalMesh);
			SaveGeometryCache(*TaskResource->Cache, *SkeletalMesh, TaskResource->SimulatedPositions);
			SavePackage(*TaskResource->Cache);
		}
		if (bCancelled)
		{
			TaskResource->Notification->SetProgressText(LOCTEXT("Cancelled", "Cancelled"));
			TaskResource->Notification->SetComplete(false);
		}
		else
		{
			TaskResource->Notification->SetProgressText(LOCTEXT("Finished", "Finished"));
			TaskResource->Notification->SetComplete(true);
		}
		TaskResource.Reset();
	}
};

#undef LOCTEXT_NAMESPACE
