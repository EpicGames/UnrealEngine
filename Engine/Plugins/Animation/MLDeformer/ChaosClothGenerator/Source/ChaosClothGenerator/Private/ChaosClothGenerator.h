// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Async/AsyncWork.h"
#include "TickableEditorObject.h"
#include "UObject/StrongObjectPtr.h"

DECLARE_LOG_CATEGORY_EXTERN(LogChaosClothGenerator, Log, All);

class UGeometryCache;
class UClothGeneratorProperties;
namespace UE::Chaos::ClothGenerator
{
	class FClothGeneratorProxy;

	enum class EClothGeneratorActions
	{
		NoAction,
		StartGenerate,
		TickGenerate
	};

	class FChaosClothGenerator : public FTickableEditorObject
	{
	public:
		FChaosClothGenerator();
		virtual ~FChaosClothGenerator();

		//~ Begin FTickableEditorObject Interface
		virtual void Tick(float DeltaTime) override;
		virtual TStatId GetStatId() const override;
		virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
		//~ End FTickableEditorObject Interface

		UClothGeneratorProperties& GetProperties() const;
		void RequestAction(EClothGeneratorActions Action);
	private:
		struct FSimResource;
		struct FTaskResource;
		class FLaunchSimsTask;
		// Using dummy TTaskRunner to avoid adding declaration of FTaskResource and FLaunchSimsTask to header file
		template<typename TaskType>
		class TTaskRunner;

		using FProxy = FClothGeneratorProxy;
		using FExecuterType = FAsyncTask<TTaskRunner<FLaunchSimsTask>>;

		void StartGenerate();
		void TickGenerate();
		void FreeTaskResource(bool bCancelled);
		UGeometryCache* GetCache() const;

		TStrongObjectPtr<UClothGeneratorProperties> Properties;
		EClothGeneratorActions PendingAction = EClothGeneratorActions::NoAction;
		TUniquePtr<FTaskResource> TaskResource;	
	};
};