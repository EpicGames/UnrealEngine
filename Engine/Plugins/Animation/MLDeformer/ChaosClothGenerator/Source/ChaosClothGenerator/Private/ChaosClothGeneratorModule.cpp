// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosClothGeneratorToolsMenuExtender.h"
#include "MLDeformerEditorToolkit.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SClothGeneratorWidget.h"

namespace UE::Chaos::ClothGenerator
{
	class FChaosClothGeneratorModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override
		{
			UE::MLDeformer::FMLDeformerEditorToolkit::AddToolsMenuExtender(CreateToolsMenuExtender());
			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.RegisterCustomClassLayout("ClothGeneratorProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FClothGeneratorDetails::MakeInstance));
		}

		virtual void ShutdownModule() override
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.UnregisterCustomClassLayout("ClothGeneratorProperties");
		}
	};
};

IMPLEMENT_MODULE(UE::Chaos::ClothGenerator::FChaosClothGeneratorModule, ChaosClothGenerator)
