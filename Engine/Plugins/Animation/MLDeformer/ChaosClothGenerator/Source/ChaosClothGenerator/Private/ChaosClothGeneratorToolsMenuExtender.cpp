// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosClothGeneratorToolsMenuExtender.h"

#include "ClothGeneratorProperties.h"
#include "Engine/SkeletalMesh.h"
#include "Framework/Docking/TabManager.h"
#include "IDocumentation.h"
#include "MLDeformerGeomCacheEditorModel.h"
#include "MLDeformerEditorModel.h"
#include "MLDeformerModel.h"
#include "MLDeformerGeomCacheModel.h"
#include "SClothGeneratorWidget.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "ChaosClothGeneratorToolsMenuExtender"

namespace UE::Chaos::ClothGenerator
{
	namespace Private
	{
		void SpawnTab(UE::MLDeformer::FMLDeformerEditorToolkit& Toolkit)
		{
			TSharedPtr<SDockTab> Tab = Toolkit.GetAssociatedTabManager()->TryInvokeTab(FName(FChaosClothGeneratorTabSummoner::TabID));
			if (!Tab.IsValid())
			{
				return;
			}
			
			TSharedRef<SWidget> TabContent = Tab->GetContent();
			if (TabContent == SNullWidget::NullWidget || TabContent->GetTypeAsString() != "SClothGeneratorWidget")
			{
				return;
			}
			TSharedRef<SClothGeneratorWidget> Widget = StaticCastSharedRef<SClothGeneratorWidget>(Tab->GetContent());
			if (Widget == SNullWidget::NullWidget)
			{
				return;
			}
		
			TWeakObjectPtr<UClothGeneratorProperties> Properties = Widget->GetProperties();
			if (!Properties.IsValid())
			{
				return;
			}
		
			using UE::MLDeformer::FMLDeformerEditorModel;
			if (FMLDeformerEditorModel* EditorModel = Toolkit.GetActiveModel())
			{
				if (UMLDeformerModel* Model = EditorModel->GetModel())
				{
					Properties->SkeletalMeshAsset = Model->GetSkeletalMesh();
					Properties->AnimationSequence = EditorModel->GetActiveTrainingInputAnimSequence();
					if (UMLDeformerGeomCacheModel* GeomCacheModel = Cast<UMLDeformerGeomCacheModel>(Model))
					{
						const UE::MLDeformer::FMLDeformerGeomCacheEditorModel* GeomCacheEditorModel = static_cast<const UE::MLDeformer::FMLDeformerGeomCacheEditorModel*>(EditorModel);
						if (UGeometryCache* GeomCache = GeomCacheEditorModel->GetActiveGeometryCache())
						{
							Properties->SimulatedCache = GeomCache;
						}
					}
				}
			}
		}
	};
	
	TUniquePtr<FChaosClothGeneratorToolsMenuExtender> CreateToolsMenuExtender()
	{
		return MakeUnique<FChaosClothGeneratorToolsMenuExtender>();
	}
	
	const FName FChaosClothGeneratorTabSummoner::TabID = FName("ChaosClothGenerator");
	
	FChaosClothGeneratorTabSummoner::FChaosClothGeneratorTabSummoner(const TSharedRef<FMLDeformerEditorToolkit>& InEditor)
		: FWorkflowTabFactory(TabID, InEditor)
	{
		bIsSingleton = true;
		TabLabel = LOCTEXT("ChaosClothGenerator", "Chaos Cloth Generator");
		ViewMenuDescription = LOCTEXT("ViewMenu_Desc", "Chaos Cloth Generator");
		ViewMenuTooltip = LOCTEXT("ViewMenu_ToolTip", "Show the Chaos Cloth Generator tab.");
	}
	
	TSharedRef<SWidget> FChaosClothGeneratorTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
	{
		return SNew(SClothGeneratorWidget);
	}
	
	TSharedPtr<SToolTip> FChaosClothGeneratorTabSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
	{
		return IDocumentation::Get()->CreateToolTip(
			LOCTEXT("ChaosClothGeneratorWidgetTooltip", "Generate training data using chaos cloth solver."),
			NULL,
			TEXT("Shared/Editors/Persona"),
			TEXT("ChaosClothGenerator_Window"));
	}
	
	FMenuEntryParams FChaosClothGeneratorToolsMenuExtender::GetMenuEntry(FMLDeformerEditorToolkit& Toolkit) const
	{
		FMenuEntryParams Params;
		Params.DirectActions = FUIAction(
			FExecuteAction::CreateLambda([&Toolkit]()
			{
				Private::SpawnTab(Toolkit);
			}),
			FCanExecuteAction::CreateLambda([]()
			{
				return true;
			})
		);
		Params.LabelOverride = LOCTEXT("ChaosClothGenerator", "Chaos Cloth Generator");
		Params.ToolTipOverride = LOCTEXT("ChaosClothGeneratorMenuTooltip", "Generate training data using chaos cloth solver");
		return Params;
	}
	
	TSharedPtr<FWorkflowTabFactory> FChaosClothGeneratorToolsMenuExtender::GetTabSummoner(const TSharedRef<FMLDeformerEditorToolkit>& Toolkit) const
	{
		return MakeShared<FChaosClothGeneratorTabSummoner>(Toolkit);
	}
};

#undef LOCTEXT_NAMESPACE