// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerEditorToolkit.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

namespace UE::Chaos::ClothGenerator
{
	struct FChaosClothGeneratorTabSummoner : public FWorkflowTabFactory
	{
	public:
		static const FName TabID;
		using FMLDeformerEditorToolkit = UE::MLDeformer::FMLDeformerEditorToolkit;
	
		FChaosClothGeneratorTabSummoner(const TSharedRef<FMLDeformerEditorToolkit>& InEditor);
		virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
		virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override;
	};
	
	class FChaosClothGeneratorToolsMenuExtender : public UE::MLDeformer::FToolsMenuExtender
	{
	public:
		using FMLDeformerEditorToolkit = UE::MLDeformer::FMLDeformerEditorToolkit;
		~FChaosClothGeneratorToolsMenuExtender() override = default;
		virtual FMenuEntryParams GetMenuEntry(FMLDeformerEditorToolkit& Toolkit) const override;
		virtual TSharedPtr<FWorkflowTabFactory> GetTabSummoner(const TSharedRef<FMLDeformerEditorToolkit>& Toolkit) const;
	};

	TUniquePtr<FChaosClothGeneratorToolsMenuExtender> CreateToolsMenuExtender();
};