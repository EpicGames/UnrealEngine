// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothGeneratorComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothGeneratorComponent)

namespace UE::Chaos::ClothGenerator
{
    FClothGeneratorProxy::FClothGeneratorProxy(const UChaosClothComponent& InClothComponent)
	    : FClothSimulationProxy(InClothComponent)
    {	
    }

    FClothGeneratorProxy::~FClothGeneratorProxy() = default;

	void FClothGeneratorProxy::PostConstructor()
	{
		constexpr bool bAsyncInitialization = false;
		PostConstructorInternal(bAsyncInitialization);
	}
};

UClothGeneratorComponent::UClothGeneratorComponent(const FObjectInitializer& ObjectInitializer)
    : Super(ObjectInitializer)
{
}

UClothGeneratorComponent::UClothGeneratorComponent(FVTableHelper& Helper)
    : Super(Helper)
{
}

UClothGeneratorComponent::~UClothGeneratorComponent() = default;

TSharedPtr<UE::Chaos::ClothAsset::FClothSimulationProxy> UClothGeneratorComponent::CreateClothSimulationProxy()
{
    TSharedPtr<FProxy> ProxyShared = MakeShared<FProxy>(*this);
    Proxy = ProxyShared;
    return ProxyShared;
}

void UClothGeneratorComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
    UActorComponent::TickComponent(DeltaTime, TickType, ThisTickFunction);
}


TWeakPtr<UE::Chaos::ClothGenerator::FClothGeneratorProxy> UClothGeneratorComponent::GetProxy() const
{
    return Proxy;
}

void UClothGeneratorComponent::Pose(const TArray<FTransform>& InComponentSpaceTransforms)
{
	if (!ensure(InComponentSpaceTransforms.Num() == GetComponentSpaceTransforms().Num()))
	{
		return;
	}
	GetEditableComponentSpaceTransforms() = InComponentSpaceTransforms;
	bNeedToFlipSpaceBaseBuffers = true;
	FinalizeBoneTransform();

	UpdateBounds();
    if (IsInGameThread())
    {
        MarkRenderTransformDirty();
        MarkRenderDynamicDataDirty();
    }
}
