// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosClothAsset/ClothComponent.h"
#include "ChaosClothAsset/ClothSimulationProxy.h"

#include "ClothGeneratorComponent.generated.h"

namespace UE::Chaos::ClothGenerator
{
	class FClothGeneratorProxy : public UE::Chaos::ClothAsset::FClothSimulationProxy
    {
    public:
        using FClothSimulationProxy = UE::Chaos::ClothAsset::FClothSimulationProxy;
        explicit FClothGeneratorProxy(const UChaosClothComponent& InClothComponent);
        ~FClothGeneratorProxy();

		virtual void PostConstructor() override;

        using FClothSimulationProxy::Tick;
        using FClothSimulationProxy::PreProcess_Internal;
        using FClothSimulationProxy::InitializeConfigs;
        using FClothSimulationProxy::WriteSimulationData;
    };
};

/**
 * Cloth data generation component.
 */
UCLASS()
class UClothGeneratorComponent : public UChaosClothComponent
{
	GENERATED_BODY()	
public:
	UClothGeneratorComponent(const FObjectInitializer& ObjectInitializer);
	UClothGeneratorComponent(FVTableHelper& Helper);
	~UClothGeneratorComponent();

    using FProxy = UE::Chaos::ClothGenerator::FClothGeneratorProxy;
    TWeakPtr<FProxy> GetProxy() const;
    
	/** Pose the cloth component using component space transforms. */
    void Pose(const TArray<FTransform>& InComponentSpaceTransforms);
protected:
	//~ Begin UActorComponent Interface
	virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	//~ End UActorComponent Interface

    /** Begin UChaosClothComponent Interface */
	virtual TSharedPtr<UE::Chaos::ClothAsset::FClothSimulationProxy> CreateClothSimulationProxy() override;
    /** End UChaosClothComponent Interface */
private:
    TWeakPtr<FProxy> Proxy;
};
