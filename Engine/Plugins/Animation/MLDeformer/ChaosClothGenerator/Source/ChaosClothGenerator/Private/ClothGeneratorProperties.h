// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ChaosClothAsset/ClothAsset.h"
#include "Animation/AnimSequence.h"
#include "Engine/SkinnedAsset.h"
#include "GeometryCache.h"

#include "ClothGeneratorProperties.generated.h"

UCLASS()
class UClothGeneratorProperties : public UObject
{
	GENERATED_BODY()
public:
	/* Skeletal mesh that will be used in MLDeformer */
	UPROPERTY(EditAnywhere, Category = Input)
	TObjectPtr<USkinnedAsset> SkeletalMeshAsset;

	/* Chaos cloth asset used in simulation. This should be different from the skeletal mesh asset. */
	UPROPERTY(EditAnywhere, Category = Input)
	TObjectPtr<UChaosClothAsset> ClothAsset;

	/* Training poses. */
	UPROPERTY(EditAnywhere, Category = Input)
	TObjectPtr<UAnimSequence> AnimationSequence;

	/* e.g. "0, 2, 5-10, 12-15". If left empty, all frames will be used */
	UPROPERTY(EditAnywhere, Category = Input)
	FString FramesToSimulate;

	/* Output meshes */
	UPROPERTY(EditAnywhere, Category = Output, meta = (EditCondition = "!bDebug"))
	TObjectPtr<UGeometryCache> SimulatedCache;

	/* Debug a single pose */
	UPROPERTY(EditAnywhere, Category = Debug)
	bool bDebug = false;

	/* The frame to inspect */
	UPROPERTY(EditAnywhere, Category = Debug, meta = (Min = 0, EditCondition = "bDebug"))
	uint32 DebugFrame = 0;

	/* The output sequence */
	UPROPERTY(EditAnywhere, Category = Debug, meta = (EditCondition = "bDebug"))
	TObjectPtr<UGeometryCache> DebugCache;

	UPROPERTY(EditAnywhere, Category = "Simulation Settings", meta = (Min = 0))
	float TimeStep = 1.f / 30;

	UPROPERTY(EditAnywhere, Category = "Simulation Settings", meta = (Min = 0))
	int32 NumSteps = 200;

	UPROPERTY(EditAnywhere, Category = "Simulation Settings", meta = (Min = 1, EditCondition = "!bDebug"))
	int32 NumThreads = 1;
};