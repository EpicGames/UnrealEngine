// Copyright Epic Games, Inc. All Rights Reserved.

#include "SClothGeneratorWidget.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "ClothGeneratorProperties.h"
#include "ChaosClothGenerator.h"
#include "ContentBrowserModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IContentBrowserSingleton.h"
#include "IDetailsView.h"
#include "Misc/FileHelper.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ClothGeneratorWidget"

namespace UE::Chaos::ClothGenerator
{
	void SClothGeneratorWidget::Construct(const FArguments& InArgs)
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		ChaosClothGenerator = MakeShared<FChaosClothGenerator>();

		FDetailsViewArgs Args;
		DetailsView = PropertyModule.CreateDetailView(Args);
		DetailsView->SetObject(&ChaosClothGenerator->GetProperties());

		this->ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				DetailsView.ToSharedRef()
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Center)
			[
				SNew(SBox)
				.WidthOverride(300)
				[
					SNew(SButton)
					.Text(FText::FromString("Start Generating"))
					.HAlign(HAlign_Center)
					.OnClicked_Lambda([this]() -> FReply
					{
						ChaosClothGenerator->RequestAction(EClothGeneratorActions::StartGenerate);
						return FReply::Handled();
					})
				]
			]
		];
	}

	TWeakObjectPtr<UClothGeneratorProperties> SClothGeneratorWidget::GetProperties() const
	{
		if (ChaosClothGenerator.IsValid())
		{
			return &ChaosClothGenerator->GetProperties();
		}
		return nullptr;
	}

	TSharedRef<IDetailCustomization> FClothGeneratorDetails::MakeInstance()
	{
		return MakeShareable(new FClothGeneratorDetails);
	}

	namespace Private
	{
		template<class T>
		T* CreateOrLoad(const FString& PackageName)
		{
			const FName AssetName(FPackageName::GetLongPackageAssetName(PackageName));		
			if (UPackage* const Package = CreatePackage(*PackageName))
			{
				LoadPackage(nullptr, *PackageName, LOAD_Quiet | LOAD_EditorOnly);
				T* Asset = FindObject<T>(Package, *AssetName.ToString());
				if (!Asset)
				{
					Asset = NewObject<T>(Package, *AssetName.ToString(), RF_Public | RF_Standalone | RF_Transactional);
					Asset->MarkPackageDirty();
					FAssetRegistryModule::AssetCreated(Asset);
				}
				return Asset;
			}
			return nullptr;
		}

		TObjectPtr<UGeometryCache> NewGeometryCacheDialog(const UObject* NamingAsset = nullptr)
		{
			FSaveAssetDialogConfig Config;
			{
				if (NamingAsset)
				{
					const FString PackageName = NamingAsset->GetOutermost()->GetName();
					Config.DefaultPath = FPackageName::GetLongPackagePath(PackageName);
					Config.DefaultAssetName = FString::Printf(TEXT("GC_%s"), *NamingAsset->GetName());
				}
				Config.AssetClassNames.Add(UGeometryCache::StaticClass()->GetClassPathName());
				Config.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::Disallow;
				Config.DialogTitleOverride = LOCTEXT("ExportGeometryCacheDialogTitle", "Export Geometry Cache As");
			}
				
			FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				
			FString NewPackageName;
			FText OutError;
			for (bool bFilenameValid = false; !bFilenameValid; bFilenameValid = FFileHelper::IsFilenameValidForSaving(NewPackageName, OutError))
			{
				const FString AssetPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(Config);
				if (AssetPath.IsEmpty())
				{
					return nullptr;
				}
				NewPackageName = FPackageName::ObjectPathToPackageName(AssetPath);
			}
			return CreateOrLoad<UGeometryCache>(NewPackageName);
		}

		TWeakObjectPtr<UClothGeneratorProperties> GetProperties(IDetailLayoutBuilder& DetailBuilder)
		{
			TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
			DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
			if (ObjectsBeingCustomized.IsEmpty())
			{
				return nullptr;
			}
			else
			{
				return Cast<UClothGeneratorProperties>(ObjectsBeingCustomized[0]);
			}
		}

		void AddGeometryCacheRowWithButton(IDetailLayoutBuilder& DetailBuilder, IDetailCategoryBuilder& CategoryBuilder, const FName PropertyName)
		{
			TSharedPtr<IPropertyHandle> Property = DetailBuilder.GetProperty(PropertyName);
			DetailBuilder.HideProperty(Property);
			CategoryBuilder.AddCustomRow(FText::FromName(PropertyName))
			.NameContent()
			[
				Property->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					Property->CreatePropertyValueWidget()
				]
				+ SHorizontalBox::Slot()
				.MaxWidth(100)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.Text(FText::FromString("New"))
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.OnClicked_Lambda([Property, &DetailBuilder]() -> FReply
					{
						TWeakObjectPtr<UClothGeneratorProperties> Properties = Private::GetProperties(DetailBuilder);
						const UObject* const NamingAsset = Properties.IsValid() ? Properties->SkeletalMeshAsset.Get() : nullptr;
						UGeometryCache* const NewGeometryCache = Private::NewGeometryCacheDialog(NamingAsset);
						Property->SetValue(NewGeometryCache);
						return FReply::Handled();
					})
				]
			];
		}
	}


	void FClothGeneratorDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
	{
		IDetailCategoryBuilder& InputCategory = DetailBuilder.EditCategory("Input");
		IDetailCategoryBuilder& OutputCategory = DetailBuilder.EditCategory("Output");
		Private::AddGeometryCacheRowWithButton(DetailBuilder, OutputCategory, GET_MEMBER_NAME_CHECKED(UClothGeneratorProperties, SimulatedCache));

		IDetailCategoryBuilder& DebugCategory = DetailBuilder.EditCategory("Debug");
		DebugCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UClothGeneratorProperties, bDebug));
		DebugCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UClothGeneratorProperties, DebugFrame));
		Private::AddGeometryCacheRowWithButton(DetailBuilder, DebugCategory, GET_MEMBER_NAME_CHECKED(UClothGeneratorProperties, DebugCache));
	}
};

#undef LOCTEXT_NAMESPACE