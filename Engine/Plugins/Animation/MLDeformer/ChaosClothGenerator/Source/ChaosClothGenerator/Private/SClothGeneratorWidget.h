// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Widgets/SCompoundWidget.h"

class UClothGeneratorProperties;
class IDetailsView;
namespace UE::Chaos::ClothGenerator
{
	class FChaosClothGenerator;

	class SClothGeneratorWidget : public SCompoundWidget
	{
		SLATE_BEGIN_ARGS(SClothGeneratorWidget) 
		{}
		SLATE_END_ARGS()

	public:
		void Construct(const FArguments& InArgs);
		TWeakObjectPtr<UClothGeneratorProperties> GetProperties() const;

	private:
		TSharedPtr<IDetailsView> DetailsView;
		TSharedPtr<FChaosClothGenerator> ChaosClothGenerator;
	};

	class FClothGeneratorDetails : public IDetailCustomization
	{
	public:
		static TSharedRef<IDetailCustomization> MakeInstance();
		virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	};
};